*/Binaries/*
*/Intermediate/*
*/.git